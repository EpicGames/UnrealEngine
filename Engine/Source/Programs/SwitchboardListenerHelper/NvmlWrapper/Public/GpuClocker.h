// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdint.h>

#include "NvmlWrapperPublic.h"

/** Encapsulates the logic to lock/unlock gpu clocks */
class EXPORTLIB FGpuClocker
{
public:

	/** Locks the gpu clocks to the maximum allowed */
	bool LockGpuClocks();

	/** Unlocks the gpu and memory clocks */
	bool UnlockGpuClocks();

	/** Locks the given gpu and memory clocks to the maximum allowed */
	bool GpuUnlockDevice(const uint32_t GpuIdx);

	/** Unlocks the given gpu and memory clocks */
	bool GpuLockDevice(const uint32_t GpuIdx);
};

