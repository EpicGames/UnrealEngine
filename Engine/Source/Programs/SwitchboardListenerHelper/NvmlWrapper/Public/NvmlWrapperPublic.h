// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(_MSC_VER)
	#define EXPORTLIB __declspec(dllexport)
	#define IMPORTLIB __declspec(dllimport)
#elif defined(__GNUC__)
	#define EXPORTLIB __attribute__((visibility("default")))
	#define IMPORTLIB
#else
	#define EXPORTLIB
	#define IMPORTLIB
#endif
