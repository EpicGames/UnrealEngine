// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBLHelperService.h"
#include "SwitchboardListenerHelper.h"

#include "Common/TcpListener.h"
#include "Tasks/Task.h"

#include "Dom/JsonValue.h"
#include "JsonObjectConverter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"


namespace UE::SBLHelperService
{
	/** LAN broadcast address.*/
	static const FIPv4Endpoint InvalidEndpoint(FIPv4Address::LanBroadcast, 0);

	/** Default timeout for inactive clients before considering them stale */
	const float DefaultInactiveTimeoutSeconds = 5;
}

FSBLHelperService::~FSBLHelperService()
{
	Stop();
}

void FSBLHelperService::Start(uint16 Port)
{
	// Start from scratch every time.
	Stop();

	bIsRunning = true;

	FIPv4Endpoint Endpoint;
	const FString HostName = FString::Printf(TEXT("localhost:%d"), Port);
	FIPv4Endpoint::FromHostAndPort(*HostName, Endpoint);

	SocketListener = MakeUnique<FTcpListener>(Endpoint, FTimespan::FromSeconds(1), false /*bInReusable*/);

	if (SocketListener->IsActive())
	{
		SocketListener->OnConnectionAccepted().BindRaw(this, &FSBLHelperService::OnIncomingConnection);
		UE_LOGF(LogSwitchboardListenerHelper, Display, "Started listening on %ls:%d", *SocketListener->GetLocalEndpoint().Address.ToString(), SocketListener->GetLocalEndpoint().Port);
	}
}

bool FSBLHelperService::OnIncomingConnection(FSocket* InSocket, const FIPv4Endpoint& InEndpoint)
{
	UE_LOGF(LogSwitchboardListenerHelper, Display, "Incoming connection via %ls:%d", *InEndpoint.Address.ToString(), InEndpoint.Port);

	PendingConnections.Enqueue(TPair<FIPv4Endpoint, TSharedPtr<FSocket>>(InEndpoint, MakeShareable(InSocket)));

	return true;
}

void FSBLHelperService::Stop()
{
	if (SocketListener.IsValid())
	{
		SocketListener->Stop();
		SocketListener = nullptr;
	}
}

bool FSBLHelperService::IsRunning()
{
	return bIsRunning;
}

bool FSBLHelperService::ParseIncomingMessage(const FString& Message, const FIPv4Endpoint& InEndpoint)
{
	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(Message);

	TSharedPtr<FJsonObject> JsonData;

	if (!FJsonSerializer::Deserialize(Reader, JsonData))
	{
		return false;
	}

	TSharedPtr<FJsonValue> CommandField = JsonData->TryGetField(TEXT("cmd")); // e.g. lock/unlock

	if (!CommandField.IsValid())
	{
		return false;
	}

	const FString Command = CommandField->AsString();

	if (Command.Equals(TEXT("lock")))
	{
		return HandleCmdLock(JsonData);
	}
	else if (Command.Equals(TEXT("nop")))
	{
		return true;
	}
	else if (Command.Equals(TEXT("bye")))
	{
		// The client wishes to disconnect immediately
		PendingDisconnections.Enqueue(InEndpoint);

		return true;
	}
	else
	{
		UE_LOGF(LogSwitchboardListenerHelper, Error, "Unsupported command '%ls'", *Command);
	}

	return false;
}

bool FSBLHelperService::HandleCmdLock(const TSharedPtr<FJsonObject>& Json)
{
	TSharedPtr<FJsonValue> PidField = Json->TryGetField(TEXT("pid"));

	if (!PidField.IsValid())
	{
		UE_LOGF(LogSwitchboardListenerHelper, Error, "Could not get valid pid from lock command");
		return false;
	}

	const uint32 Pid = uint32(FCString::Atoi64(*PidField->AsString()));

	// Get a handle to the process from the Pid
	FProcHandle ProcHandle = FPlatformProcess::OpenProcess(Pid);

	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogSwitchboardListenerHelper, Warning, "Received Gpu clock lock request for Pid %u, but could not open a handle to it", Pid);
		return false;
	}

	UE_LOGF(LogSwitchboardListenerHelper, Display, "Received Gpu clock lock request for Pid %u", Pid);
	++NumProcessesHoldingGpuClocks;

	// Now we issue a background task to wait for the process to exit
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [ProcHandle, this]()
		{
			FProcHandle ProcHandleCopy = ProcHandle;

			// This should only return when the process stops.
			FPlatformProcess::WaitForProc(ProcHandleCopy);

			// Decrement the number of holding gpu clocks.
			--NumProcessesHoldingGpuClocks;
		});

	return true;
}

FString FSBLHelperService::CreateMessage(const FString& Cmd, const TMap<FString, FString>& AdditionalFields)
{
	FString Message;

	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter
		= TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Message);

	JsonWriter->WriteObjectStart();

	// message id
	JsonWriter->WriteValue(TEXT("id"), FString::Printf(TEXT("%u"), GetNextPacketId()));

	// cmd
	JsonWriter->WriteValue(TEXT("cmd"), Cmd);

	for (const auto& Value : AdditionalFields)
	{
		JsonWriter->WriteValue(Value.Key, Value.Value);
	}

	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	return Message;
}

uint32 FSBLHelperService::GetNextPacketId()
{
	NextPacketId++;

	// Avoid 0, which is reserved for no packet id.

	if (NextPacketId == 0)
	{
		NextPacketId = 1;
	}

	return NextPacketId;
}

bool FSBLHelperService::SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::SendMessage);

	if (InEndpoint == UE::SBLHelperService::InvalidEndpoint)
	{
		return false;
	}

	if (!Connections.Contains(InEndpoint))
	{
		// this happens when a client disconnects while a task it had issued is not finished
		UE_LOGF(LogSwitchboardListenerHelper, Verbose, "Trying to send message to disconnected client %ls", *InEndpoint.ToString());

		return false;
	}

	TSharedPtr<FSocket> ClientSocket = Connections[InEndpoint];

	if (!ClientSocket.IsValid())
	{
		return false;
	}

	// Send the message over the socket. 
	// Note the "+ 1" to include the null terminator, which is the message delimiter in the SBLHelper protocol.

	UE_LOGF(LogSwitchboardListenerHelper, Verbose, "Sending message %ls", *InMessage);

	int32 BytesSent = 0;
	return ClientSocket->Send((uint8*)TCHAR_TO_UTF8(*InMessage), InMessage.Len() + 1, BytesSent);
}

void FSBLHelperService::HandlePendingConnections()
{
	TPair<FIPv4Endpoint, TSharedPtr<FSocket>> Connection;

	while (PendingConnections.Dequeue(Connection))
	{
		Connections.Add(Connection);

		const FIPv4Endpoint ClientEndpoint = Connection.Key;

		LastActivityTime.FindOrAdd(ClientEndpoint, FPlatformTime::Seconds());

		// Send hello message upon connection. The client will be able to verify the version of this server.
		{
			const FString Cmd = TEXT("hello");

			TMap<FString, FString> AdditionalFields;
			AdditionalFields.Emplace(TEXT("sblhver"), FString::Printf(TEXT("%d.%d.%d"),
				SBLHELPER_VERSION_MAJOR,
				SBLHELPER_VERSION_MINOR,
				SBLHELPER_VERSION_PATCH
			));

			SendMessage(CreateMessage(Cmd, AdditionalFields), ClientEndpoint);
		}
	}
}

void FSBLHelperService::HandlePendingDisconnections()
{
	const double CurrentTime = FPlatformTime::Seconds();

	for (const TPair<FIPv4Endpoint, double>& LastActivity : LastActivityTime)
	{
		const FIPv4Endpoint& Client = LastActivity.Key;
		const float ClientTimeoutSeconds = UE::SBLHelperService::DefaultInactiveTimeoutSeconds;

		if ((CurrentTime - LastActivity.Value) > ClientTimeoutSeconds)
		{
			UE_LOGF(LogSwitchboardListenerHelper, Display, "Client %ls has been inactive for more than %.1fs", 
				*Client.ToString(), 
				ClientTimeoutSeconds
			);

			PendingDisconnections.Enqueue(Client);
		}
	}

	FIPv4Endpoint Client;

	while (PendingDisconnections.Dequeue(Client))
	{
		DisconnectClient(Client);
	}
}

void FSBLHelperService::DisconnectClient(const FIPv4Endpoint& InClientEndpoint)
{
	const FString Client = InClientEndpoint.ToString();
	UE_LOGF(LogSwitchboardListenerHelper, Display, "Disconnected from Client %ls", *Client);

	Connections.Remove(InClientEndpoint);
	LastActivityTime.Remove(InClientEndpoint);
	ReceiveBuffer.Remove(InClientEndpoint);
}

void FSBLHelperService::ParseIncomingData()
{
	for (const TPair<FIPv4Endpoint, TSharedPtr<FSocket>>& Connection : Connections)
	{
		const FIPv4Endpoint& ClientEndpoint = Connection.Key;
		const TSharedPtr<FSocket>& ClientSocket = Connection.Value;

		uint32 PendingDataSize = 0;
		while (ClientSocket->HasPendingData(PendingDataSize))
		{
			LastActivityTime[ClientEndpoint] = FPlatformTime::Seconds();

			TArray<uint8> Buffer;
			Buffer.AddUninitialized(PendingDataSize);

			int32 BytesRead = 0;

			if (!ClientSocket->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
			{
				UE_LOGF(LogSwitchboardListenerHelper, Error, "Error while receiving data via endpoint %ls", *ClientEndpoint.ToString());
				continue;
			}

			TArray<uint8>& MessageBuffer = ReceiveBuffer.FindOrAdd(ClientEndpoint);

			for (int32 ByteIdx = 0; ByteIdx < BytesRead; ++ByteIdx)
			{
				MessageBuffer.Add(Buffer[ByteIdx]);

				// We use '\0' as message terminator
				if (Buffer[ByteIdx] == '\x00')
				{
					const FString Message(UTF8_TO_TCHAR(MessageBuffer.GetData()));
					ParseIncomingMessage(Message, ClientEndpoint);
					MessageBuffer.Empty();
				}
			}
		}
	}
}

void FSBLHelperService::ManageGpuClocks()
{
	if (NumProcessesHoldingGpuClocks)
	{
		if (!bGpuClocksLocked)
		{
			UE_LOGF(LogSwitchboardListenerHelper, Display, "Locking Gpu Clocks");
			GpuClocker.LockGpuClocks();
			bGpuClocksLocked = true;
		}
	}
	else
	{
		if (bGpuClocksLocked)
		{
			UE_LOGF(LogSwitchboardListenerHelper, Display, "Unlocking Gpu Clocks");
			GpuClocker.UnlockGpuClocks();
			bGpuClocksLocked = false;
		}
	}
}

void FSBLHelperService::Tick()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListenerHelper::Tick);

	// Dequeue pending connections
	HandlePendingConnections();

	// Parse incoming data from remote connections
	ParseIncomingData();

	// Manage Gpu clocks
	ManageGpuClocks();

	// Close stale sockets
	HandlePendingDisconnections();
}

