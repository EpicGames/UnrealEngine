// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "GpuClocker.h"

class FJsonObject;
class FTcpListener;

/** Provides services that Switchboard listener cannot perform on its own due to required user privileges */
class FSBLHelperService
{
public:

	~FSBLHelperService();

	/** Starts the service and tcp server */
	void Start(uint16 Port = 8010);

	/** Stops the service and tcp server */
	void Stop();

	/** Returns true if the service is currently running or not.*/
	bool IsRunning();
	
	/** Call periodically to perform service maintenance tasks */
	void Tick();

private:

	/** Called when there is a new tcp client connection */
	bool OnIncomingConnection(FSocket* InSocket, const FIPv4Endpoint& InEndpoint);

	/** 
	 * Parses and handles the specified incoming message 
	 * 
	 * @param Message The message to parse and handle.
	 * @param InEndpoint The endpoint that sent the given message.
	 * 
	 * @return True if the message was parsed and handled
	 */
	bool ParseIncomingMessage(const FString& Message, const FIPv4Endpoint& InEndpoint);

	/**
	 * Creates a new message for the given command and additional fields
	 *
	 * @param Cmd The command to create this message for.
	 * @param AdditionalFields Any additional fields or parameters to include in the message.
	 *
	 * @return The created message, ready to be sent over the socket.
	 */
	FString CreateMessage(const FString& Cmd, const TMap<FString, FString>& AdditionalFields);

	/** Gets the next packet id to use when creating a new message. Monotonically increments. Skips zero. */
	uint32 GetNextPacketId();

	/** Send the given message to the given endpoint */
	bool SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint);

	/** Handles the lock gpu clock command */
	bool HandleCmdLock(const TSharedPtr<FJsonObject>& Json);

	/** Dequeues the pending new tcp client connections and sends a hello message */
	void HandlePendingConnections();

	/** Parses all incoming data from all connected clients */
	void ParseIncomingData();

	/** Locks/unlocks the gpu clocks depending on the Pids that are holding said request */
	void ManageGpuClocks();

	/** Removes stale client connections and clients who sent a bye cmd */
	void HandlePendingDisconnections();

	/** Removes the given client endpoint from the list of active connections and frees related resources */
	void DisconnectClient(const FIPv4Endpoint& InClientEndpoint);

private:

	/** List of incoming client connections that have not been handled yet */
	TQueue<TPair<FIPv4Endpoint, TSharedPtr<FSocket>>, EQueueMode::Spsc> PendingConnections;

	/** List of active client connections */
	TMap<FIPv4Endpoint, TSharedPtr<FSocket>> Connections;

	/** List of client connections to be disconnected (due to inactivity or bye command) */
	TQueue<FIPv4Endpoint, EQueueMode::Spsc> PendingDisconnections;

	/** Receive buffers dedicated to each client */
	TMap<FIPv4Endpoint, TArray<uint8>> ReceiveBuffer;

	/** Flag indicating whether the service is currently running or not */
	bool bIsRunning = false;

	/** Next packet id when creating a new message to be sent */
	uint32 NextPacketId = 1;

	/** The listening tcp socket */
	TUniquePtr<FTcpListener> SocketListener;

	/** Count of processes that are holding gpu clock lock active */
	std::atomic<int32> NumProcessesHoldingGpuClocks = 0;

	/** Whether the gpu clocks have been locked by this service or not */
	bool bGpuClocksLocked = false;

	/** Last activity time for each client. Used to detect stale connections. */
	TMap<FIPv4Endpoint, double> LastActivityTime;

	/** Used to lock/unlock the gpu clocks on demand */
	FGpuClocker GpuClocker;

};

