// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListenerHelper.h"


#include "Common/TcpListener.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "RequiredProgramMainCPPInclude.h"
#include "SBLHelperService.h"
#include "Stats/StatsSystem.h"

IMPLEMENT_APPLICATION(SwitchboardListenerHelper, "SwitchboardListenerHelper");
DEFINE_LOG_CATEGORY(LogSwitchboardListenerHelper);


namespace UE::SwitchboardListenerHelper
{
	/** Initializes the Engine */
	static int32 InitEngine(const TCHAR* InCommandLine)
	{
		const int32 InitResult = GEngineLoop.PreInit(InCommandLine);
		if (InitResult != 0)
		{
			return InitResult;
		}

		ProcessNewlyLoadedUObjects();
		FModuleManager::Get().StartProcessingNewlyLoadedObjects();

		// Load internal plugins in the pre-default phase
		IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);

		// Load plugins in default phase
		IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

		return 0;
	}

	/** Loads the socket module */
	static bool InitSocketSystem()
	{
		EModuleLoadResult LoadResult;
		FModuleManager::Get().LoadModuleWithFailureReason(TEXT("Sockets"), LoadResult);

		FIPv4Endpoint::Initialize();

		return LoadResult == EModuleLoadResult::Success;
	}

	/** Clean teardown of the Engine */
	static void EngineTearDown()
	{
		// Cleanup. See also: FEngineLoop::AppPreExit and FEngineLoop::AppExit

		RequestEngineExit(TEXT("Shutting down SwitchboardListenerHelper"));

		FCoreDelegates::OnPreExit.Broadcast();
		FCoreDelegates::OnExit.Broadcast();
		FModuleManager::Get().UnloadModulesAtShutdown();

#if STATS
		FThreadStats::StopThread();
#endif

		FTaskGraphInterface::Shutdown();
		FPlatformMisc::PlatformTearDown();

		if (GLog)
		{
			GLog->TearDown();
		}

		FTextLocalizationManager::TearDown();
		FInternationalization::TearDown();

		FTraceAuxiliary::Shutdown();
	}

	/** Called when the window is closed or interrupted */
	static void HandleApplicationWillTerminate()
	{
		RequestEngineExit(TEXT("HandleApplicationWillTerminate"));
	}

	/** This is the main loop of the program */
	static void MainLoop()
	{
		// We need a chance to handle service being terminated and not leave the GPU clocks locked if not desired.
		FCoreDelegates::GetApplicationWillTerminateDelegate().AddStatic(&HandleApplicationWillTerminate);

		double LastTime = FPlatformTime::Seconds();
		const float MinFrameTimeSeconds = 0.1f;

		// Start the service
		FSBLHelperService Service;
		{
			uint16 Port = 8010; // Default tcp port
			uint16 PortOverride = Port;

			if (FParse::Value(FCommandLine::Get(), TEXT("sblhport="), PortOverride))
			{
				Port = PortOverride;
			}

			Service.Start(Port);
		}

		while (Service.IsRunning() && !IsEngineExitRequested())
		{
			const double CurrentTime = FPlatformTime::Seconds();
			const double DeltaTime = CurrentTime - LastTime;
			LastTime = CurrentTime;

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

			// Pump & Tick objects
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			// Tick our service
			Service.Tick();

			GFrameCounter++;
			UE::Stats::FStats::AdvanceFrame(false);
			GLog->FlushThreadedLogs();

			// Run garbage collection for the UObjects for the rest of the frame or at least to 10 ms
			IncrementalPurgeGarbage(true, FMath::Max<float>(0.010f, MinFrameTimeSeconds - (FPlatformTime::Seconds() - CurrentTime)));

			// Throttle main thread main fps by sleeping if we still have time
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, MinFrameTimeSeconds - (FPlatformTime::Seconds() - CurrentTime)));
		}

		Service.Stop();
	}

	/** Initializes the Engine, calls the Main loop, then performs teardown when exiting the application */
	static int32 RunSwitchboardListenerHelper(int32 ArgC, TCHAR* ArgV[])
	{
		// Populate the CommandLine from the input argument.
		const FString CommandLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
		FCommandLine::Set(*CommandLine);

		// Initialize the Engine
		{
			const int32 InitResult = InitEngine(*CommandLine);

			UE_LOGF(LogSwitchboardListenerHelper, Display, "SwitchboardListenerHelper %u.%u.%u", SBLHELPER_VERSION_MAJOR, SBLHELPER_VERSION_MINOR, SBLHELPER_VERSION_PATCH);
			UE_LOGF(LogSwitchboardListenerHelper, Display, "NOTE: This process should be run with elevated privileges to operate properly.");

			if (InitResult != 0)
			{
				UE_LOGF(LogSwitchboardListenerHelper, Fatal, "Could not initialize engine, Error code: %d", InitResult);
				RequestEngineExit(TEXT("Engine init failure"));
				return InitResult;
			}

			if (!InitSocketSystem())
			{
				UE_LOGF(LogSwitchboardListenerHelper, Fatal, "Could not initialize socket system!");
				RequestEngineExit(TEXT("Socket init failure"));
				return 1;
			}
		}

#if PLATFORM_WINDOWS
		// Always minimize on launch
		ShowWindow(GetConsoleWindow(), SW_MINIMIZE);
#endif

		// Call main loop of this application
		MainLoop();

		// Clean tear down of the Engine
		EngineTearDown();

		return 0;
	}
}

/** This is the program's entry point */
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	using namespace UE::SwitchboardListenerHelper;

	int32 ExitCode;
	if (FPlatformMisc::IsDebuggerPresent())
	{
		ExitCode = RunSwitchboardListenerHelper(ArgC, ArgV);
	}
	else
	{
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
		{
			// SetCrashHandler(nullptr) sets up default behavior for Linux and Mac interfacing with CrashReportClient
			FPlatformMisc::SetCrashHandler(nullptr);
			GIsGuarded = true;
			ExitCode = RunSwitchboardListenerHelper(ArgC, ArgV);
			GIsGuarded = false;
		}
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except (ReportCrash(GetExceptionInformation()))
		{
			if (GError)
			{
				GError->HandleError();
			}

			// RequestExit(bForce==True) terminates this process with a specific exit code and does not return.
			FPlatformMisc::RequestExit(true);
			ExitCode = 1; // Suppress compiler warning.
		}
#endif
	}

	return ExitCode;
}
