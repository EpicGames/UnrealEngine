// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

#define SBLHELPER_VERSION_MAJOR 1
#define SBLHELPER_VERSION_MINOR 0
#define SBLHELPER_VERSION_PATCH 0

DECLARE_LOG_CATEGORY_EXTERN(LogSwitchboardListenerHelper, Verbose, All);

