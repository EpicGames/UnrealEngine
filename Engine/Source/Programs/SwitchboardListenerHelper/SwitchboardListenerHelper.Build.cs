// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class SwitchboardListenerHelper : ModuleRules
{
	public SwitchboardListenerHelper(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore", // for LaunchEngineLoop.cpp dependency
				"Core",
				"CoreUObject",
				"Json",
				"Networking",
				"Projects", // for LaunchEngineLoop.cpp dependency
				"JsonUtilities",
				"TraceLog",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "NvmlWrapper", "Lib", "Windows", "NvmlWrapper.lib"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "NvmlWrapper", "Lib", "Linux", "libNvmlWrapper.a"));
		}

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
			}
		);

		PrivateIncludePaths.AddRange(
			new string[] {
				"Runtime/Launch/Private", // for LaunchEngineLoop.cpp include
			}
		);

		PublicIncludePaths.AddRange(
			new string[]
			{
				Path.Combine(ModuleDirectory, "NvmlWrapper", "Public"),
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Sockets",
			}
		);
	}
}
