// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;
using EpicGames.Core;
using UnrealBuildBase;

public class SwitchboardListenerHelperTarget : TargetRules
{
	public SwitchboardListenerHelperTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "SwitchboardListenerHelper";
		SolutionDirectory = "Programs/Switchboard";

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		// This app compiles against Core/CoreUObject, but not the Engine or Editor, so compile out Engine and Editor references from Core/CoreUObject
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildWithEditorOnlyData = false;

		// Disable internationalization support, so the exe can be launched outside of the engine directory.
		bCompileICU = false;

		// The listener is meant to be a console application (no window), but on MacOS, to get a proper log console, a full application must be built.
		bIsBuildingConsoleApplication = Target.Platform != UnrealTargetPlatform.Mac;

		bEnableTrace = true;

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// Run with elevated privileges.

			AdditionalLinkerArguments = "/MANIFESTUAC:NO";
			WindowsPlatform.ManifestFile = FileReference.Combine(Unreal.EngineDirectory,
				"Source", "Programs", "SwitchboardListenerHelper", "Resources", "Windows",
				"SBLHelper.manifest.xml").ToString();
		}
	}
}
